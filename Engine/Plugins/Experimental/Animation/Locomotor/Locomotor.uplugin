{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Locomotor",
	"Description": "Procedural animation for Control Rig.",
	"Category": "Animation",
	"CreatedBy": "Epic Games",
	"CreatedByURL": "www.epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": true,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Plugins": [
		{
			"Name": "ControlRig",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "Locomotor",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		}
	]
}