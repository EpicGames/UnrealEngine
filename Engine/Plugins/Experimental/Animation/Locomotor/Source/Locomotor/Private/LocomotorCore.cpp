// Copyright Epic Games, Inc. All Rights Reserved.

#include "LocomotorCore.h"

#include "Engine/World.h"
#include "Components/PrimitiveComponent.h"
#include "CollisionQueryParams.h"
#include "CollisionShape.h"

//#pragma optimize("", off)

FLocomotorFoot::FLocomotorFoot(
	const FTransform& InitialWorldRootGoal,
	const FTransform& InitialWorldFoot,
	const FLocomotorFootSettings& InSettings)
{
	Settings = InSettings;
	CurrentPhase = 1.0f;
	FootPhaseWhenSwingEnds = 0.5f;
	InitialRelativeToRootGoal = InitialWorldFoot.GetRelativeTransform(InitialWorldRootGoal);
	InitialRelativeToRootGoal.NormalizeRotation();
	InitialRelativeToBody = FTransform::Identity; // this is initialized after Pelvis calculates initial body transform
	InitialWorld = InitialWorldFoot;
	CurrentWorld = InitialWorld;
	CurrentWorldFlatPosition = InitialWorld.GetLocation();
	CurrentWorldFlatPositionNoEase = InitialWorld.GetLocation();
	CurrentTargetWorld = InitialWorld;
	PrevWorld = InitialWorld;
	CurrentRotationSpring.Reset(InitialWorld.GetRotation());
	PlantedWorld = InitialWorld;
	FinalTargetWorld = InitialWorld;
	StrideOriginWorld = InitialWorld;
	CurrentHeightOffset = 0.f;
	bAtRest = true;
	bInSwingPhase = false;
	bWantsToStep = false;
}

int32 FLocomotorFootSet::AddFoot(
	const FTransform& InitialWorldRootGoal,
	const FTransform& InitialWorldFoot,
	const FLocomotorFootSettings& InSettings)
{
	const int32 FootIndexInSet = Feet.Emplace(InitialWorldRootGoal, InitialWorldFoot, InSettings);
	FLocomotorFoot& NewFoot = Feet[FootIndexInSet];
	NewFoot.Settings.StaticPhaseOffset = FootIndexInSet % 2 == 0.f ? 0.f : 0.5f;
	NewFoot.Settings.FootSetIndex = SetIndex;
	return FootIndexInSet;
}

void FLocomotorHead::Initialize(const FTransform& InInitialWorld)
{
	InitialWorld = InInitialWorld;
	bInitialized = true;
}

void FLocomotorPelvis::Initialize(const TArray<FLocomotorFoot*>& AllFeet, const FTransform& InitialRootGoalWorld)
{
	CurrentWorld = InitialWorld; // initial pelvis world set when Reset()'ing the locomotor
	CalculateBodyTransform(AllFeet, InitialRootGoalWorld, InitialBodyWorld);
	CurrentBodyWorld = CurrentBodyTargetWorld = PreviousBodyTargetWorld = InitialBodyWorld;

	BodyLeadDamper.Reset(FVector::ZeroVector); // initialized to zero because we smooth the lead amount, not absolute value
	BodyRotationSpring.Reset(InitialBodyWorld.GetRotation());

	CurrentHeightOffset = InitialWorld.GetLocation().Z;
	BobOffsetSpring.Reset(0.f);
}

void FLocomotorPelvis::CalculateBodyTransform(
	const TArray<FLocomotorFoot*> AllFeet,
	const FTransform& RootGoal,
	FTransform& OutBodyTransform)
{
	FVector AvgFootPosition = FVector::ZeroVector;
	for (const FLocomotorFoot* FootPtr : AllFeet)
	{
		AvgFootPosition += FootPtr->CurrentWorldFlatPositionNoEase;
	}
	const float InvNumFeet = 1.0f / AllFeet.Num();
	AvgFootPosition *= InvNumFeet;

	OutBodyTransform.SetRotation(RootGoal.GetRotation());
	OutBodyTransform.SetTranslation(AvgFootPosition);
}

void FLocomotor::Reset(const FTransform& InInitialRootGoalWorld, const FTransform& InInitialWorldPelvis)
{
	InitialRootGoalWorld = InInitialRootGoalWorld;
	Pelvis.InitialWorld = InInitialWorldPelvis;
	Pelvis.PelvisPositionDamper.Reset(InInitialWorldPelvis.GetLocation());
	
	AllFeet.Reset();
	FootSets.Reset();
	CurrentGlobalPhase = 0.0f;
	CurrentSpeed = 0.0f;
	CurrentPhaseSpeed = 0.0f;
	CurrentPercentOfMaxSpeed = 0.0f;
	bFullyAtRest = true;
	bAccelerating = true;
	AccumulatedTimeToSimulate = 0.0f;

	bPostInitialized = false;
}

int32 FLocomotor::AddFootSet(const float PhaseOffset)
{
	bPostInitialized = false;
	float WrappedPhaseOffset = FMath::Fmod(FMath::Fmod(PhaseOffset, 1.0f) + 1.0f, 1.0f);
	return FootSets.Emplace(FLocomotorFootSet(WrappedPhaseOffset, FootSets.Num()));
}

int32 FLocomotor::AddFootToSet(
	const int32 FootSetIndex,
	const FTransform& InitialWorldFoot,
	const FLocomotorFootSettings& InSettings)
{
	bPostInitialized = false;
	
	if (!FootSets.IsValidIndex(FootSetIndex))
	{
		return INDEX_NONE;
	}
	
	const int32 FootIndexInSet = FootSets[FootSetIndex].AddFoot(InitialRootGoalWorld, InitialWorldFoot, InSettings);

	// update cached pointers to all feet
	// NOTE: we reset and update all pointers here in case the foot arrays inside the sets
	// were reallocated which would invalidate all the pointers in the cache.
	AllFeet.Reset(0);
	for (FLocomotorFootSet& FootSet : FootSets)
	{
		for (FLocomotorFoot& Foot : FootSet.GetFeet())
		{
			AllFeet.Add(&Foot);
		}
	}
	
	return FootIndexInSet;
}

void FLocomotor::SetHead(const FTransform& InitialWorldTransform)
{
	Head.Initialize(InitialWorldTransform);
}

void FLocomotor::RunSimulation(const FLocomotorInputSettings& InSettings)
{
	// run post initialization and validation
	if (!Initialize())
	{
		return;
	}

	// copy latest input settings
	Settings = InSettings;

	// run simulation at max internal time step...
	// this breaks large time steps into multiple smaller simulation steps for robustness
	constexpr float MaxTimeStep = 1 / 120.0f;
	constexpr float MinTimeStep = MaxTimeStep / 4.f;
	// delta time is accumulated across updates, allows for extreme slow motion
	AccumulatedTimeToSimulate += InSettings.DeltaTime;
	while (AccumulatedTimeToSimulate > MinTimeStep)
	{
		Settings.DeltaTime = AccumulatedTimeToSimulate >= MaxTimeStep ? MaxTimeStep : AccumulatedTimeToSimulate;
		AccumulatedTimeToSimulate -= Settings.DeltaTime;
		Simulate();
	}
}

void FLocomotor::GetFeetCurrent(TArray<FTransform> OutTransforms) const
{
	// OutTransforms must be sized by calling code
	if (!ensure(OutTransforms.Num() == AllFeet.Num()))
	{
		return;
	}
	
	for (int32 FootIndex = 0; FootIndex < AllFeet.Num(); ++FootIndex)
	{
		OutTransforms[FootIndex] = AllFeet[FootIndex]->CurrentWorld;
	}
}

void FLocomotor::Simulate()
{
	UpdateWorldSpeedAndPhase();
	UpdateFeetTargets();
	AnimateFeet();
	UpdateBody();
	AnimatePelvis();
}

bool FLocomotor::Initialize()
{
	if (!bPostInitialized)
	{
		Pelvis.Initialize(AllFeet, InitialRootGoalWorld);

		// requires Pelvis.Initialize() to be called first to get InitialBodyWorld
		for (FLocomotorFoot* Foot : AllFeet)
		{
			Foot->InitialRelativeToBody = Foot->InitialWorld.GetRelativeTransform(Pelvis.InitialBodyWorld);
		}
	
		bPostInitialized = true;
	}
	
	return !GetFeet().IsEmpty();
}

void FLocomotor::UpdateWorldSpeedAndPhase()
{
	//
	// apply acceleration and deceleration
	//

	// how far away is the pelvis from it's target location?
	const FTransform InitialPelvisRelativeToRootGoal = Pelvis.InitialWorld.GetRelativeTransform(InitialRootGoalWorld);
	const FTransform CurrentPelvisAttachedToRootGoal = InitialPelvisRelativeToRootGoal * Settings.CurrentWorldRootGoal;
	const float DistanceToGoal = FVector::Distance(CurrentPelvisAttachedToRootGoal.GetLocation(), Pelvis.CurrentWorld.GetLocation());
	const bool GoalFurtherThanSingleStride = DistanceToGoal > Settings.Movement.MinimumStepLength * 2.0f;
	
	// how long would it take to slow down to minimum speed?
	// calculate distance to reach minimum velocity from current velocity (given current deceleration)
	const float CurrentStoppingDistance =
		(FMath::Pow(Settings.Movement.SpeedMin, 2) - FMath::Pow(CurrentSpeed, 2))	/ (-Settings.Movement.Deceleration * 2.f);
	const bool bGoalBeyondStoppingDistance = DistanceToGoal > CurrentStoppingDistance;

	// determine if body is rotating a significant amount
	//const float AngularDistanceRadians = InitialRootGoalWorld.GetRotation().AngularDistance(Settings.CurrentWorldRootGoal.GetRotation());
	//constexpr float AngularThresholdForFullAcceleration = 40.0f;
	//const bool bNeedsFastRotation = FMath::RadiansToDegrees(AngularDistanceRadians) > AngularThresholdForFullAcceleration;

	// determine whether we are speeding up or slowing down based current speed and distance to the target
	bAccelerating = GoalFurtherThanSingleStride && bGoalBeyondStoppingDistance;// || bNeedsFastRotation;
	const float Acceleration = bAccelerating ? Settings.Movement.Acceleration : -Settings.Movement.Deceleration;
	// integrate the acceleration for this time step
	CurrentSpeed = CurrentSpeed + Acceleration * Settings.DeltaTime;
	// clamp within bounds
	CurrentSpeed = FMath::Clamp(CurrentSpeed, Settings.Movement.SpeedMin, Settings.Movement.SpeedMax);
	// where are we in the range betweeen min and max speed
	CurrentPercentOfMaxSpeed = FMath::GetRangePct(Settings.Movement.SpeedMin, Settings.Movement.SpeedMax, CurrentSpeed);
	CurrentPercentOfMaxSpeed = FMath::Clamp(CurrentPercentOfMaxSpeed,0.f, 1.f);
	CurrentPhaseSpeed = FMath::Lerp(Settings.Movement.PhaseSpeedMin, Settings.Movement.PhaseSpeedMax, CurrentPercentOfMaxSpeed);
	CurrentStrideLength = CurrentSpeed / CurrentPhaseSpeed;
	
	// reset phase if fully at rest
	if (bFullyAtRest)
	{
		// stop updating at rest
		CurrentGlobalPhase = 0.0f;
	}
	else
	{
		// update the global phase
		CurrentGlobalPhase += CurrentPhaseSpeed * Settings.DeltaTime;
		CurrentGlobalPhase = WrapPhaseInRange(CurrentGlobalPhase);
	}

	// DEBUG OUTPUT
	//UE_LOGF(LogTemp, Warning, "Distance to Goal  : %f", DistanceToGoal);
	//UE_LOGF(LogTemp, Warning, "Stopping Distance : %f", CurrentStoppingDistance);
	//UE_LOGF(LogTemp, Warning, "Current speed     : %f", CurrentSpeed);
	//UE_LOGF(LogTemp, Warning, "Phase   speed     : %f", CurrentPhaseSpeed);
	//UE_LOGF(LogTemp, Warning, "Delta time        : %f", Settings.DeltaTime);
	//UE_LOGF(LogTemp, Warning, "PercentOfMaxSpeed : %f", CurrentPercentOfMaxSpeed);
	//UE_LOGF(LogTemp, Warning, "Current Stride    : %f", CurrentStrideLength);
	//UE_LOGF(LogTemp, Warning, "Accelerating      : %ls", bAccelerating ? TEXT("Accelerating") : TEXT("Decelerating"));
}

void FLocomotor::UpdateFeetTargets()
{
	// continuously update final targets for each foot
	// these are the foot locations at the root goal location
	for (FLocomotorFoot* FootPtr : AllFeet)
	{
		FLocomotorFoot& Foot = *FootPtr;
		
		// continuously update the final foot target
		Foot.FinalTargetWorld = Foot.InitialRelativeToRootGoal * Settings.CurrentWorldRootGoal;
		
		if (Settings.Movement.bTeleport)
		{
			Foot.CurrentWorld = Foot.PlantedWorld = Foot.CurrentTargetWorld = Foot.FinalTargetWorld;
			Foot.bAtRest = true;
			Foot.bInSwingPhase = Foot.bUnplantingThisTick = Foot.bWantsToStep = false;
			continue;
		}
		
		if (Settings.Stepping.bEnableGroundCollision)
		{
			ProjectToGroundWithSphereCast(Foot, Foot.FinalTargetWorld);	
		}

		// vector from prev planted location to final target
		Foot.PrevToFinalTargetNorm = Foot.FinalTargetWorld.GetLocation() - Foot.PlantedWorld.GetLocation();
		Foot.PrevToFinalTargetDistance = Foot.PrevToFinalTargetNorm.Length();
		Foot.PrevToFinalTargetNorm.Normalize();
	}

	if (Settings.Movement.bTeleport)
	{
		return;
	}

	// continuously update "bWantsToStep"
	// (based on if the foot's final target is beyond the min stride distance)
	bool bAnyFeetWantToStep = false;
	for (FLocomotorFoot* FootPtr : AllFeet)
	{
		FLocomotorFoot& Foot = *FootPtr;
		// wants to step if the distance to the target is greater than the minimum stride length
		Foot.bWantsToStep = Foot.PrevToFinalTargetDistance > Settings.Movement.MinimumStepLength;
		if (Foot.bWantsToStep)
		{
			bAnyFeetWantToStep = true;
		}
	}
	
	// if we are fully at rest and any foot wants to step,
	// then we need to reset the global phase to the closest foot AND reset each foot's current phase
	if (bFullyAtRest && bAnyFeetWantToStep)
	{
		// find the foot that is furthest from the target
		float FurthestDistance = -1.0f;
		int32 FurthestFootIndex = INDEX_NONE;
		for (int32 FootIndex=0; FootIndex<AllFeet.Num(); ++FootIndex)
		{
			const FLocomotorFoot* Foot = AllFeet[FootIndex];
			if (Foot->PrevToFinalTargetDistance > FurthestDistance)
			{
				FurthestDistance = Foot->PrevToFinalTargetDistance;
				FurthestFootIndex = FootIndex;
			}
		}

		// rewind/fast-forward the current phase so that the furthest foot steps first
		CurrentGlobalPhase = WrapPhaseInRange(-AllFeet[FurthestFootIndex]->Settings.StaticPhaseOffset);

		// force rewind/fast-forward all foot phases
		for (FLocomotorFoot* FootPtr : AllFeet)
		{
			FLocomotorFoot& Foot = *FootPtr;
			Foot.CurrentPhase = Foot.TargetPhase = WrapPhaseInRange(CurrentGlobalPhase + Foot.Settings.StaticPhaseOffset);
			Foot.FootPhaseWhenSwingEnds = 0.5f;
			//UE_LOGF(LogTemp, Warning, "Force updating feet phases: %f", Foot.CurrentPhase);
		}
	}

	// update "bAtRest", "bInSwingPhase" and "bUnplantingThisTick" states
	for (FLocomotorFoot* FootPtr : AllFeet)
	{
		FLocomotorFoot& Foot = *FootPtr;
		
		// record if this foot was planted before this tick
		const bool bWasInSwing = Foot.bInSwingPhase;
		// check if the foot is now in the swing phase
		Foot.bInSwingPhase = Foot.CurrentPhase < Foot.FootPhaseWhenSwingEnds;
		// if the foot is in it's planted phase, AND the target is not beyond the min threshold, then we can consider this foot "at rest"
		Foot.bAtRest = !Foot.bInSwingPhase && !Foot.bWantsToStep;
		// is this foot unplanting this tick?
		Foot.bUnplantingThisTick = !bWasInSwing && Foot.bInSwingPhase; 
	}

	// update "CurrentPhase" and "PhaseWhenSwingEnds"
	const float PhaseCatchUpPerSecond = CurrentPhaseSpeed * 2.0f; 
	const float PhaseBlendRate = PhaseCatchUpPerSecond * Settings.DeltaTime;
	for (FLocomotorFoot* FootPtr : AllFeet)
	{
		FLocomotorFoot& Foot = *FootPtr;
		
		// update feet phases
		if (Foot.bAtRest)
		{
			Foot.CurrentPhase = Foot.TargetPhase = 1.0f;
		}
		else
		{
			const int32 FootSetIndex = FMath::Clamp(Foot.Settings.FootSetIndex, 0, FootSets.Num() - 1);
			const float PhaseOffsetOfSet = FootSets[FootSetIndex].GetPhaseOffset();
			const float StaticOffsetOfFoot = Foot.Settings.StaticPhaseOffset;
			Foot.TargetPhase = WrapPhaseInRange(CurrentGlobalPhase + StaticOffsetOfFoot + PhaseOffsetOfSet);
			Foot.CurrentPhase = BlendTowardsTargetPhase(Foot.CurrentPhase, Foot.TargetPhase, PhaseBlendRate);
			//UE_LOGF(LogTemp, Warning, "Updating foot phase Current/Target: %f, %f", Foot.CurrentPhase, Foot.TargetPhase);
		}
		
		// if unplanting this tick, we need to calculate the phase when swing ends (locked it in for the duration of the step)
		if (Foot.bUnplantingThisTick)
		{
			Settings.Stepping.PercentOfStrideInAir = FMath::Clamp(Settings.Stepping.PercentOfStrideInAir, 0.1f, 0.95f);
			Settings.Stepping.AirExtensionAtMaxSpeed = FMath::Clamp(Settings.Stepping.AirExtensionAtMaxSpeed,0.0f, 0.95f);
			
			const float PercentStrideInAirAtMinSpeed = Settings.Stepping.PercentOfStrideInAir;
			const float PercentStrideInAirAtMaxSpeed = FMath::Clamp(
				PercentStrideInAirAtMinSpeed + Settings.Stepping.AirExtensionAtMaxSpeed,
				Settings.Stepping.PercentOfStrideInAir,
				0.95f);

			Foot.FootPhaseWhenSwingEnds = FMath::Lerp(
				PercentStrideInAirAtMinSpeed,
				PercentStrideInAirAtMaxSpeed,
				CurrentPercentOfMaxSpeed);
		}
	}
	
	// continuously update the current target of each foot that is in a swing phase
	for (FLocomotorFoot* FootPtr : AllFeet)
	{
		FLocomotorFoot& Foot = *FootPtr;
		
		if (!Foot.bInSwingPhase)
		{
			continue;
		}

		// some foot parameters are updated once at the start of a stride (when unplanting)
		if (Foot.bUnplantingThisTick)
		{
			// update stride origin when lifting off
			// NOTE: we don't use the Foot.PlantedWorldPosition because it may be behind where the foot's neutral pose can reach
			Foot.StrideOriginWorld = Foot.InitialRelativeToBody * Pelvis.CurrentBodyWorld;
			ProjectToGroundWithSphereCast(Foot, Foot.StrideOriginWorld);

			// update foot height for the current stride
			// NOTE: this is not done continuously because it can lead to foot height wobbling if the speed changes mid step
			const float MinStrideHeight = Settings.Stepping.StepHeight * 0.2f;
			const float MaxStrideHeight = Settings.Stepping.StepHeight;
			Foot.CurrentStrideHeight = FMath::Lerp(MinStrideHeight, MaxStrideHeight, CurrentPercentOfMaxSpeed);
		}

		// generate a candidate target transform for the foot
		FTransform NewTarget = FTransform::Identity;
		
		// target location is in direct line from stride origin to final target, scaled by step length
		const FVector StrideOriginToFinalTarget = Foot.FinalTargetWorld.GetLocation() - Foot.StrideOriginWorld.GetLocation();
		const float StepLengthToUse = FMath::Min(StrideOriginToFinalTarget.Length(), CurrentStrideLength);
		NewTarget.SetTranslation(Foot.StrideOriginWorld.GetLocation() + StrideOriginToFinalTarget.GetSafeNormal() * StepLengthToUse);

		// (optionally) project the candidate target onto the ground and update orientation based on normal
		if (Settings.Stepping.bEnableGroundCollision)
		{
			FVector GroundNormal = ProjectToGroundWithSphereCast(Foot, NewTarget);

			// update pitch/roll orientation of the foot based on current target ground normal

			// decompose normal into pitch and roll rotations
			const FVector Fwd = Settings.CurrentWorldRootGoal.GetRotation().RotateVector(FVector(0,1,0));
			const FVector Up = FVector(0,0,1);
			FVector Side;
			FQuat PitchRotation;
			FQuat RollRotation;
			CalcPitchRollFromNormal(Fwd, Up, GroundNormal, Side,PitchRotation, RollRotation);

			// scale rotations by user amount
			PitchRotation = FQuat::FastLerp(FQuat::Identity, PitchRotation, Settings.Stepping.OrientFootToGroundPitch);
			RollRotation = FQuat::FastLerp(FQuat::Identity, RollRotation, Settings.Stepping.OrientFootToGroundRoll);

			// apply to the current target
			NewTarget.SetRotation(PitchRotation * RollRotation * Foot.FinalTargetWorld.GetRotation());
		}
		else
		{
			// target rotation is simply the FINAL target rotation
			// while the location of the feet takes many steps to reach the final target,
			// the rotation of the feet should get there in a single step.
			NewTarget.SetRotation(Foot.FinalTargetWorld.GetRotation());
		}
		
		// now continuously update the target as the phase progresses
		// NOTE:
		//	- we allow the FIRST target after un-planting to be FULLY applied, but afterwards we limit target updates by foot speed
		//	- this roughly simulates "committing" to the step and the inability to change direction on a dime.
		if (Foot.bUnplantingThisTick)
		{
			// retarget target spring at start of stride
			Foot.CurrentTargetSpring.Reset(NewTarget.GetLocation());
		}
		else
		{
			// when the character is slowing down, new targets are pulled closer which may make them further from the final target
			// we should never update a target to make it further from the goal
			const float OldTargetToFinalDistSq = FVector::DistSquared(Foot.FinalTargetWorld.GetLocation(), Foot.CurrentTargetWorld.GetLocation());
			const float NewTargetToFinalDistSq = FVector::DistSquared(Foot.FinalTargetWorld.GetLocation(), NewTarget.GetLocation());
			if (NewTargetToFinalDistSq < OldTargetToFinalDistSq)
			{
				// move the current target towards the new target with a damped spring
				// NOTE: target is on a damped spring so it can continuously update throughout the stride, but without sharp discontinuities
				Foot.CurrentTargetSpring.Update(Settings.DeltaTime, NewTarget.GetLocation(), 10.0f, 2.0f);
				NewTarget.SetTranslation(Foot.CurrentTargetSpring.GetCurrent());
			}
			else
			{
				// the continuously updated target was further away than the previous one, so simply ignore it
				NewTarget.SetTranslation(Foot.CurrentTargetWorld.GetLocation());
			}
		}
		
		// set the current target for this foot
		Foot.CurrentTargetWorld = NewTarget;
	}

	// run foot collision to prevent overlapping feet targets
	if (Settings.Stepping.bEnableFootCollision)
	{
		// resolve collisions between each foot and every other foot
		for (FLocomotorFoot* FootPtr : AllFeet)
		{
			ResolveFootToFootCollision(*FootPtr);
		}
	}

	// last minute cancel footstep if the generated target is too close to the planted foot
	// NOTE: this can happen if the ground and/or foot collision moved the target
	// so close to the planted foot that the step would be smaller than some fraction of the minimum threshold step size
	const float FootStepLengthThreshold = FMath::Pow(Settings.Movement.MinimumStepLength * 0.5f, 2);
	for (FLocomotorFoot* FootPtr : AllFeet)
	{
		FLocomotorFoot& Foot = *FootPtr;
		if (Foot.bInSwingPhase)
		{
			const float StepLengthSq = FVector::DistSquared(Foot.PlantedWorld.GetLocation(), Foot.CurrentTargetWorld.GetLocation());
			if (StepLengthSq < FootStepLengthThreshold)
			{
				// cancel this step
				Foot.bAtRest = true;
				Foot.bInSwingPhase = false;
				Foot.CurrentPhase = Foot.TargetPhase = 1.0f;
			}
		}
	}

	// check if we are fully at rest
	bFullyAtRest = true;
	for (const FLocomotorFoot* FootPtr : AllFeet)
	{
		if (!FootPtr->bAtRest)
		{
			bFullyAtRest = false;
			break;
		}
	}
}

void FLocomotor::ResolveFootToFootCollision(FLocomotorFoot& Foot)
{
	// foot at rest cannot resolve collision
	if (!Foot.bInSwingPhase)
	{
		return;
	}

	// push target away from other feet
	for (int32 OtherFootIndex=0; OtherFootIndex < AllFeet.Num(); ++OtherFootIndex)
	{
		FLocomotorFoot& OtherFoot = *AllFeet[OtherFootIndex];
		if (&Foot == &OtherFoot)
		{
			continue; // don't collide with self
		}

		// gather both feet locations and collide them against each other
		FVector CenterA = Foot.CurrentTargetWorld.GetLocation();
		FVector CenterB = OtherFoot.CurrentTargetWorld.GetLocation();
		const float RadiusA = Foot.Settings.CollisionRadius * Settings.Stepping.FootCollisionGlobalScale;
		const float RadiusB = OtherFoot.Settings.CollisionRadius * Settings.Stepping.FootCollisionGlobalScale;
		constexpr float InvMassA = 1.0f; // in swing phase
		const float InvMassB = OtherFoot.bInSwingPhase ? 1.0f : 0.0f;
		PushCirclesApartInFloorPlane(CenterA, CenterB, RadiusA, RadiusB, InvMassA, InvMassB);

		Foot.CurrentTargetWorld.SetLocation(CenterA);
		OtherFoot.CurrentTargetWorld.SetLocation(CenterB);
	}
}

void FLocomotor::PushCirclesApartInFloorPlane(
	FVector& CenterA,
	FVector& CenterB,
	float RadiusA,
	float RadiusB,
	float InvMassA,
	float InvMassB)
{
	// early out if both circles are locked in place
	const float TotalInvMass = InvMassA + InvMassB;
	if (TotalInvMass <= KINDA_SMALL_NUMBER)
	{
		return;
	}
	
	// project to floor (collision is done in 2d ground plane)
	FVector2d FloorPositionA(CenterA);
	FVector2d FloorPositionB(CenterB);
			
	// vector from this foot to other foot
	const FVector2d Delta = FloorPositionB - FloorPositionA;
	const float Distance = Delta.Size();
	const float CombinedRadii = RadiusA + RadiusB;
    
	// are the circles colliding?
	if (Distance < CombinedRadii)
	{
		const float Overlap = CombinedRadii - Distance;
		const FVector2d Direction = Delta.GetSafeNormal();

		// calculate the displacement vectors based on the inverse masses
		const FVector2d CorrectionA = -Direction * (Overlap * (InvMassA / TotalInvMass));
		const FVector2d CorrectionB = Direction * (Overlap * (InvMassB / TotalInvMass));
		// update the positions to resolve the collision
		CenterA = CenterA + FVector(CorrectionA.X, CorrectionA.Y, 0.f);
		CenterB = CenterB + FVector(CorrectionB.X, CorrectionB.Y, 0.f);
	}
}

FVector FLocomotor::ProjectToGroundWithSphereCast(const FLocomotorFoot& Foot, FTransform& TransformToProject)
{
	const FVector Offset = FVector(0, 0, FMath::Max(Settings.Stepping.MaxCollisionHeight, 0));
	const FVector Start = TransformToProject.GetLocation() + Offset;
	const FVector End = TransformToProject.GetLocation() - Offset;
	FHitResult HitResult;
	const bool bDidHit = CastSphere(Start, End, Foot.Settings.CollisionRadius, HitResult);
	if (bDidHit)
	{
		TransformToProject.SetLocation(HitResult.ImpactPoint + FVector(0,0,Foot.InitialRelativeToRootGoal.GetLocation().Z));
		return HitResult.ImpactNormal;
	}

	return FVector::ZAxisVector;
}

void FLocomotor::AnimateFeet()
{
	for (FLocomotorFoot* FootPtr : AllFeet)
	{
		FLocomotorFoot& Foot = *FootPtr;
		
		if (Foot.bAtRest || !Foot.bInSwingPhase)
		{
			// foot stays locked at current position while planted or at rest
			Foot.CurrentWorldFlatPosition = Foot.CurrentWorld.GetLocation();
			Foot.CurrentWorldFlatPositionNoEase = Foot.CurrentWorld.GetLocation();
			Foot.PrevWorld = Foot.CurrentWorld;
			Foot.PlantedWorld = Foot.CurrentWorld;
			// reset height offset
			Foot.CurrentHeightOffset = 0.f;
		}

		// update the current position while in the swing phase
		if (Foot.bInSwingPhase)
		{
			// record the previous world transform before updating the current world transform
			Foot.PrevWorld = Foot.CurrentWorld;
			
			// remap swing range from 0-PhaseWhenSwingEnds to 0-1
			float PercentStep = FMath::GetRangePct(0.f, Foot.FootPhaseWhenSwingEnds, Foot.CurrentPhase); 
			// ease in out
			float PercentStepEased = EaseInOutNorm(PercentStep, Settings.Stepping.StepEaseIn, Settings.Stepping.StepEaseOut);
			
			// move foot towards target during swing phase
			FVector PlantedPosition = Foot.PlantedWorld.GetLocation();
			FVector TargetPosition = Foot.CurrentTargetWorld.GetLocation();
			FVector NewPositionEased = FMath::Lerp(PlantedPosition, TargetPosition, PercentStepEased);
			FVector NewPositionLinear = FMath::Lerp(PlantedPosition, TargetPosition, PercentStep);
			Foot.CurrentWorldFlatPosition = NewPositionEased;
			Foot.CurrentWorldFlatPositionNoEase = NewPositionLinear; // record non-eased position as well (prevents jerk in elements reading foot locations)
			
			// add height to the current (non-flat) world transform
			Foot.CurrentHeightOffset = Foot.CurrentStrideHeight * FMath::Sin(PercentStep * PI);
			const FVector FinalHeightOffset = FVector(0.f, 0.f, Foot.CurrentHeightOffset);
			Foot.CurrentWorld.SetLocation(Foot.CurrentWorldFlatPosition + FinalHeightOffset);
		}

		// generate an animated heel peel rotation
		FQuat PeelHeelRotation = FQuat::Identity;
		if (Foot.bInSwingPhase)
		{
			// add heel peel to current (non-flat) world rotation
			constexpr float PeelDurationAsPercentOfSwing = 0.2f;
			constexpr float PeelPhaseBufferBeforeFootFall = 0.1f;
			constexpr float PhaseAtPeelStart = 0.0f;
			const float PhaseAtMaxPeel = Foot.FootPhaseWhenSwingEnds * PeelDurationAsPercentOfSwing;
			const float PhaseAtPeelEnd = Foot.FootPhaseWhenSwingEnds - PeelPhaseBufferBeforeFootFall;
			// scale the maximum peel angle by the stride length
			FQuat MaxPeelRotation = FQuat::MakeFromEuler(Foot.Settings.MaxHeelPeel);
			FQuat MinPeelRotation = FQuat::FastLerp(FQuat::Identity, MaxPeelRotation, 0.25f);
			PeelHeelRotation = FQuat::FastLerp(MinPeelRotation, MaxPeelRotation, CurrentPercentOfMaxSpeed);
			// rotate to max peel at start of phase and then back to flat foot before foot-fall
			if (Foot.CurrentPhase > PhaseAtMaxPeel)
			{
				// blend back to flat foot
				const float PercentFromMaxPeelToEndOfStride =  FMath::GetRangePct(PhaseAtMaxPeel, PhaseAtPeelEnd, Foot.CurrentPhase); 
				PeelHeelRotation = FQuat::FastLerp(PeelHeelRotation, FQuat::Identity, PercentFromMaxPeelToEndOfStride).GetNormalized();
			}
			else
			{
				// blend from flat foot to peeled heel
				const float PercentToMaxPeel =  FMath::GetRangePct(PhaseAtPeelStart, PhaseAtMaxPeel, Foot.CurrentPhase);
				PeelHeelRotation = FQuat::FastLerp(FQuat::Identity, PeelHeelRotation, PercentToMaxPeel).GetNormalized();
			}
		}

		// update current rotation through a quat spring
		// this is done regardless of what phase the foot is in (we allow foot pivoting while planted)
		{
			constexpr float RotationStiffness = 40.f;
			constexpr float RotationDamping = .9f;
			
			const FQuat NewFootRotation = Foot.CurrentRotationSpring.Update(
				Settings.DeltaTime,
				Foot.CurrentTargetWorld.GetRotation(),
				RotationStiffness,
				RotationDamping);
			Foot.CurrentWorld.SetRotation(NewFootRotation * PeelHeelRotation);
		}
	}

	// DEBUG OUTPUT
	//const FLocomotorFoot& Foot = *AllFeet[3];
	//float PercentSwing = FMath::GetRangePct(0.f, Foot.FootPhaseWhenSwingEnds, Foot.CurrentPhase);
	//float PercentSwingBasedOnFoot = FMath::GetRangePct(0.0f, Foot.FootPhaseWhenSwingEnds, Foot.CurrentPhase);
	//UE_LOGF(LogTemp, Warning, "CurrentPhase:            %f", Foot.CurrentPhase);
	//UE_LOGF(LogTemp, Warning, "PercentSwing:            %f", PercentSwing);
	//UE_LOGF(LogTemp, Warning, "PercentSwingBasedOnFoot: %f", PercentSwingBasedOnFoot);
	//UE_LOGF(LogTemp, Warning, "PhaseWhenSwingEnds:      %f", Foot.FootPhaseWhenSwingEnds);
	//UE_LOGF(LogTemp, Warning, "InSwingPhase:            %ls", Foot.bInSwingPhase ? TEXT("True") : TEXT("False"));
}

void FLocomotor::UpdateBody()
{
	if (AllFeet.IsEmpty())
	{
		// avoid divide by zero
		return;
	}
	
	// update body based on current feet
	{
		Pelvis.CalculateBodyTransform(AllFeet, Settings.CurrentWorldRootGoal, Pelvis.CurrentBodyTargetWorld);	
	}

	if (Settings.Movement.bTeleport)
	{
		Pelvis.CurrentBodyWorld = Pelvis.PreviousBodyTargetWorld = Pelvis.CurrentBodyTargetWorld;
		Pelvis.CurrentBodyLead = FVector::ZeroVector;
		Pelvis.BobOffsetSpring.Reset(0);
		Pelvis.BodyLeadDamper.Reset(FVector::ZeroVector);
		Pelvis.BodyRotationSpring.Reset(Pelvis.CurrentBodyWorld.GetRotation());
		Pelvis.CurrentHeightOffset = Pelvis.CurrentBodyTargetWorld.GetLocation().Z;
		return;
	}

	// pitch/roll the body target based on the slope of the ground
	const bool bPitchBodyWithGround = !FMath::IsNearlyZero(Settings.Pelvis.OrientToGroundPitch);
	const bool bRollBodyWithGround = !FMath::IsNearlyZero(Settings.Pelvis.OrientToGroundRoll);
	if (bPitchBodyWithGround || bRollBodyWithGround)
	{
		// get the average normal under the feet
		FVector Normal = FVector::ZeroVector;
		for (FLocomotorFoot* Foot : AllFeet)
		{
			const FVector Offset = FVector(0, 0, FMath::Max(Settings.Stepping.MaxCollisionHeight, 0));
			FVector Start = Foot->CurrentWorld.GetLocation() + Offset;
			FVector End = Foot->CurrentWorld.GetLocation() - Offset;
			FHitResult HitResult;
			const bool bDidHit = CastSphere(Start, End, Foot->Settings.CollisionRadius, HitResult);
			if (bDidHit)
			{
				Normal += HitResult.Normal;
			}
		}
		Normal.Normalize();

		// decompose normal into pitch and roll rotations
		const FVector Fwd = Settings.CurrentWorldRootGoal.GetRotation().RotateVector(FVector(0,1,0));
		const FVector Up = FVector(0,0,1);
		FVector Side;
		FQuat PitchRotation;
		FQuat RollRotation;
		CalcPitchRollFromNormal(Fwd, Up, Normal, Side,PitchRotation, RollRotation);

		// scale rotations by user amount
		PitchRotation = FQuat::FastLerp(FQuat::Identity, PitchRotation, Settings.Pelvis.OrientToGroundPitch);
		RollRotation = FQuat::FastLerp(FQuat::Identity, RollRotation, Settings.Pelvis.OrientToGroundRoll);

		// apply to the current target
		FQuat NewPelvisRotation = PitchRotation * RollRotation * Pelvis.CurrentBodyTargetWorld.GetRotation();
		Pelvis.CurrentBodyTargetWorld.SetRotation(NewPelvisRotation);
	}

	// update current body based on target + lead
	{
		// find the current velocity of the pelvis target, and use that to extrapolate the pelvis so that it leads the feet motion
		// the amount the pelvis leads the foot motion is controlled with LeadAmount.
		// the lead itself is ran through a spring damper so that the pelvis smoothly leads and returns to the target location
		const FVector TargetVelocity = Pelvis.CurrentBodyTargetWorld.GetLocation() - Pelvis.PreviousBodyTargetWorld.GetLocation();
		float LeadAmountToUse = bAccelerating ? Settings.Pelvis.LeadAmount : -Settings.Pelvis.LeadAmount;
		const FVector LeadTarget = TargetVelocity * LeadAmountToUse;
		Pelvis.CurrentBodyLead = Pelvis.BodyLeadDamper.Update(
			LeadTarget,
			Settings.DeltaTime,
			Settings.Pelvis.LeadDampingHalfLife);
		const FVector ExtrapolatedBodyPosition = Pelvis.CurrentBodyTargetWorld.GetLocation() + Pelvis.CurrentBodyLead;
		Pelvis.CurrentBodyWorld.SetLocation(ExtrapolatedBodyPosition);	
	}

	// update current body rotation through a quat spring
	{
		const FQuat NewBodyRotation = Pelvis.BodyRotationSpring.Update(
			Settings.DeltaTime,
			Pelvis.CurrentBodyTargetWorld.GetRotation(),
			Settings.Pelvis.RotationStiffness,
			Settings.Pelvis.RotationDamping);
		Pelvis.CurrentBodyWorld.SetRotation(NewBodyRotation);
	}

	// record prev target for next frame
	Pelvis.PreviousBodyTargetWorld = Pelvis.CurrentBodyTargetWorld;
}

void FLocomotor::AnimatePelvis()
{
	// update pelvis based on body motion
	{
		// THIS BROKE INPUT POSE SUPPORT, BUT THAT WAS FLIPPING ON SOME ASSETS :(
		const FTransform InitialPelvisRelativeToBody = Pelvis.InitialWorld.GetRelativeTransform(Pelvis.InitialBodyWorld);
		const FTransform CurrentWorldPelvisTarget = InitialPelvisRelativeToBody * Pelvis.CurrentBodyWorld;
		
		/*
		// generate pelvis transform based on current body transform
		const FTransform InitialRootGoalWorldNoRotation = FTransform(FQuat::Identity, InitialRootGoalWorld.GetLocation());
		const FTransform InitialBodyComponentSpace = Pelvis.InitialBodyWorld.GetRelativeTransform(InitialRootGoalWorldNoRotation);
		// use continuously updated pelvis transform (InputPelvisLocal) to support input animated pelvis 
		const FTransform InputPelvisRelativeToInitialBody = Settings.Pelvis.InputPelvisComponentSpace.GetRelativeTransform(InitialBodyComponentSpace);
		const FTransform CurrentWorldPelvisTarget = InputPelvisRelativeToInitialBody * Pelvis.CurrentBodyWorld;
		*/
		
		
		/*
		// generate pelvis transform based on current body transform
		const FTransform InitialPelvisRelativeToBody = Pelvis.InitialWorld.GetRelativeTransform(Pelvis.InitialBodyWorld);
		Pelvis.CurrentWorld = InitialPelvisRelativeToBody * Pelvis.CurrentBodyWorld;
		*/
		
		/*
		// generate pelvis transform based on current body transform
		const FTransform InitialBodyLocal = Pelvis.InitialBodyWorld.GetRelativeTransform(InitialRootGoalWorld);
		// use continuously updated pelvis transform (InputPelvisLocal) to support input animated pelvis 
		const FTransform InputPelvisRelativeToInitialBody = Settings.Pelvis.InputPelvisLocal.GetRelativeTransform(InitialBodyLocal);
		const FTransform CurrentWorldPelvisTarget = InputPelvisRelativeToInitialBody * Pelvis.CurrentBodyWorld;
		*/

		// run it through a damper
		//const FVector DampedPelvisPosition = Pelvis.PelvisPositionDamper.Update(
		//	CurrentWorldPelvisTarget.GetLocation(),
		//	Settings.DeltaTime,
		//	Settings.Pelvis.PositionDampingHalfLife);
		//Pelvis.CurrentWorld.SetLocation(DampedPelvisPosition + Pelvis.CurrentBodyLead);
		
		Pelvis.CurrentWorld = CurrentWorldPelvisTarget;
	}

	// additively apply vertical pelvis bob based on phase
	{
		// calculate average amount feet have been raised
		float TargetBobOffset = 0.f;
		for (FLocomotorFoot* FootPtr : AllFeet)
		{
			TargetBobOffset += FootPtr->CurrentHeightOffset;
		}
		TargetBobOffset /= AllFeet.IsEmpty() ? 1.0f : AllFeet.Num();

		// add a static offset when moving
		// remove static offset when foot is at rest, this removes the "double bent knee slide" when going to rest
		float NumFeetAtRest = 0.f;
		for (const FLocomotorFoot* Foot : AllFeet)
		{
			NumFeetAtRest += Foot->bAtRest ? 1.0f : 0.f;
		}
		const float PercentFeetAtRest = 1.0f - NumFeetAtRest / AllFeet.Num();
		const float ScaledStaticOffset = Settings.Pelvis.BobOffset * PercentFeetAtRest;
		TargetBobOffset += ScaledStaticOffset;
	
		// put height offset through a spring
		const float SmoothBobOffset = Pelvis.BobOffsetSpring.Update(
			Settings.DeltaTime,
			TargetBobOffset,
			Settings.Pelvis.BobStiffness,
			Settings.Pelvis.BobDamping);
		
		// apply height to pelvis
		// current location is a flat interpolation, so we can add an offset to it
		FVector CurrentLocation = Pelvis.CurrentWorld.GetLocation();
		CurrentLocation.Z += SmoothBobOffset;
		Pelvis.CurrentWorld.SetTranslation(CurrentLocation);
	}
}

void FLocomotor::AnimateHead()
{
	if (!Head.bInitialized)
	{
		return;
	}

	//Head.CurrentWorld.
}

float FLocomotor::EaseInOutNorm(const float Input, float EaseInAmount, float EaseOutAmount) const
{
	const float InputSquared = Input * Input;
	const float EasedInput =  InputSquared / (2.0f * (InputSquared - Input) + 1.0f);
	const float ClampedEaseAmount = FMath::Clamp(Input < 0.5 ? EaseInAmount : EaseOutAmount, 0.0f, 1.0f);
	return FMath::Lerp(Input, EasedInput, ClampedEaseAmount);
}

float FLocomotor::WrapPhaseInRange(float PhaseToWrap)
{
	ensure(PhaseToWrap >= -1.f);
	if (PhaseToWrap < 0.0f)
	{
		PhaseToWrap += 1.0f;
	}
	return PhaseToWrap - FMath::Floor(PhaseToWrap);
}

float FLocomotor::BlendTowardsTargetPhase(const float CurrentPhase, const float TargetPhase, const float PhaseBlendRate)
{
	// calculate distance to target in the positive direction
	float DirectDistance = TargetPhase - CurrentPhase;
	if (DirectDistance < 0.0f)
	{
		// if directDistance is negative, it means wrapping around is shorter
		DirectDistance += 1.0f;
	}

	// if the distance is greater than a 1/4 phase, pause the current phase until the target phase gets closer
	constexpr float PHASE_OFFSET_THRESHOLD = 0.25f;
	if (DirectDistance > PHASE_OFFSET_THRESHOLD)
	{
		return CurrentPhase;
	}
	
	// calculate the distance to move, this ensures we do not overshoot the target value
	const float DistanceToMove = FMath::Min(DirectDistance, PhaseBlendRate);

	// increment and wrap
	return WrapPhaseInRange(CurrentPhase + DistanceToMove);
}

float FLocomotor::GetPhaseOffsetForSetFromMovementStyle(int32 SetIndex)
{
	if (Settings.Movement.Styles.IsEmpty())
	{
		return 0.f;
	}
	
	if (!ensure(FootSets.IsValidIndex(SetIndex)))
	{
		return 0.f;
	}

	if (Settings.Movement.Styles.Num() == 1)
	{
		return 0.f;
	}

	//TODO
	return 0.f;
}

bool FLocomotor::CastSphere(const FVector& Start, const FVector& End, float SphereRadius, FHitResult& OutHitResult)
{
	if (!Settings.Stepping.World)
	{
		return false;
	}
	
	const ECollisionChannel CollisionChannel = UEngineTypes::ConvertToCollisionChannel(Settings.Stepping.TraceChannel); 
	const FCollisionShape CollisionShape = FCollisionShape::MakeSphere(SphereRadius);
	
	FCollisionQueryParams QueryParams;
	QueryParams.bTraceComplex = true;
	if (Settings.Stepping.IgnoredActor)
	{
		QueryParams.AddIgnoredActor(Settings.Stepping.IgnoredActor);
	}
	else if (const UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Settings.Stepping.IgnoredComponent))
	{
		QueryParams.AddIgnoredComponent(PrimitiveComponent);
	}
	
	FCollisionResponseParams ResponseParams(ECR_Ignore);
	ResponseParams.CollisionResponse.SetResponse(ECC_WorldStatic, ECR_Block);
	
	bool bHit = Settings.Stepping.World->SweepSingleByChannel(
		OutHitResult,
		Start,
		End,
		FQuat::Identity,
		CollisionChannel,
		CollisionShape,
		QueryParams,
		ResponseParams);

	return bHit;
}

void FLocomotor::CalcPitchRollFromNormal(
	const FVector& Fwd,
	const FVector& Up,
	const FVector& Normal,
	FVector& OutSide,
	FQuat& OutPitchRotation,
	FQuat& OutRollRotation)
{
	// decompose normal into PITCH (forward/back) and ROLL (side to side) relative to given UP and FWD vectors
	
	OutSide = Up.Cross(Fwd);

	// pitch (in forward / up plane)
	const FVector NormalInFwdUpPlane = FVector::VectorPlaneProject(Normal, OutSide).GetSafeNormal();
	OutPitchRotation = FQuat::FindBetweenNormals(Up, NormalInFwdUpPlane);
	// roll (in side / up plane)
	const FVector NormalInSideUpPlane = FVector::VectorPlaneProject(Normal, Fwd).GetSafeNormal();
	OutRollRotation = FQuat::FindBetweenNormals(Up, NormalInSideUpPlane);
};

float FLocomotor::GetPhaseCurrent() const
{
	return CurrentGlobalPhase;
}
