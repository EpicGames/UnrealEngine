// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_Locomotor.h"


#include "Kismet/KismetMathLibrary.h"
#include "Units/RigUnitContext.h"
#include "Math/ControlRigMathLibrary.h"

FRigUnit_Locomotor_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()

	// awaiting hierarchy
	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return;
	}

	// awaiting root control
	if (!RootControl.IsValid())
	{
		return;
	}

	// cache index of root control
	CachedRootElement.UpdateCache(FRigElementKey(RootControl, ERigElementType::Control), Hierarchy);

	// check if we need to reinitialize
	bool bNeedsReinitialized = !Locomotor.HasFeet();
	if (FeetTransforms.Num() != Locomotor.GetFeet().Num())
	{
		bNeedsReinitialized = true;
	}

	FTransform ToWorld = ExecuteContext.GetToWorldSpaceTransform();
	
	// initialize the simulation
	if (bNeedsReinitialized)
	{	
		// initialize with the initial root goal and initial pelvis transform
		const FTransform InitialRootGoalGlobal = Hierarchy->GetInitialGlobalTransform(CachedRootElement.GetIndex());
		const FTransform InitialPelvisGlobal = Hierarchy->GetInitialGlobalTransform(Pelvis.PelvisBone);
		const FTransform InitialPelvisWorld = InitialPelvisGlobal * ToWorld;
		const FTransform InitialRootGoalWorld = InitialRootGoalGlobal * ToWorld;
		Locomotor.Reset(InitialRootGoalWorld, InitialPelvisWorld);

		// add all the feet
		for (const FFootSet& FootSet : FootSets)
		{
			// first create a set for these feet
			int32 FootSetIndex = Locomotor.AddFootSet(FootSet.PhaseOffset);
			// add all the feet in the set
			for (const FFootSettings& Foot : FootSet.Feet)
			{
				if (Foot.AnkleBone.IsValid())
				{
					// calculate initial foot in world space
					FTransform InitialFootTransform = Hierarchy->GetInitialGlobalTransform(Foot.AnkleBone);
					FVector InitialFootTransformOffset = InitialFootTransform.GetRotation().RotateVector(Foot.StaticLocalOffset);
					InitialFootTransform.SetTranslation(InitialFootTransform.GetLocation() + InitialFootTransformOffset);
					const FTransform InitialWorldFoot = InitialFootTransform * ToWorld;

					// assemble per-foot settings
					FLocomotorFootSettings FootSettings;
					FootSettings.CollisionRadius = Foot.CollisionRadius;
					FootSettings.MaxHeelPeel = Foot.MaxHeelPeel;
					FootSettings.StaticLocalOffset = Foot.StaticLocalOffset	* ToWorld.GetScale3D();
					
					Locomotor.AddFootToSet(FootSetIndex, InitialWorldFoot, FootSettings);
				}	
			}
		}

		// create room for output
		FeetTransforms.SetNumZeroed(Locomotor.GetFeet().Num());
	}

	if (!Locomotor.HasFeet())
	{
		return;
	}

	FTransform RootTransformNoScale = Hierarchy->GetGlobalTransform(CachedRootElement.GetIndex());
	RootTransformNoScale.SetScale3D(FVector::One());
	
	// update the simulation settings
	FLocomotorInputSettings Settings;
	Settings.CurrentWorldRootGoal = RootTransformNoScale * ToWorld;
	Settings.DeltaTime = ExecuteContext.GetDeltaTime() * Movement.GlobalTimeScale;
	// movement
	Settings.Movement.MinimumStepLength = Movement.MinimumStepLength;
	Settings.Movement.SpeedMax = Movement.SpeedMax;
	Settings.Movement.SpeedMin = Movement.SpeedMin;
	Settings.Movement.PhaseSpeedMax = Movement.PhaseSpeedMax;
	Settings.Movement.PhaseSpeedMin = Movement.PhaseSpeedMin;
	Settings.Movement.Acceleration = Movement.Acceleration;
	Settings.Movement.Deceleration = Movement.Deceleration;
	for (ELocomotorMovementStyle Style : Movement.Styles)
	{
		EMovementStyle StyleToAdd = static_cast<EMovementStyle>(Style); // cast from BP enum to internal enum
		Settings.Movement.Styles.Add(StyleToAdd);
	}
	Settings.Movement.bTeleport = Movement.bTeleport;
	Settings.Stepping.PercentOfStrideInAir = Stepping.PercentOfStrideInAir;
	Settings.Stepping.AirExtensionAtMaxSpeed = Stepping.AirExtensionAtMaxSpeed;
	Settings.Stepping.StepHeight = Stepping.StepHeight;
	Settings.Stepping.StepEaseIn = Stepping.StepEaseIn;
	Settings.Stepping.StepEaseOut = Stepping.StepEaseOut;
	Settings.Stepping.bEnableFootCollision = Stepping.bEnableFootCollision;
	Settings.Stepping.FootCollisionGlobalScale = Stepping.FootCollisionGlobalScale;
	Settings.Stepping.bEnableGroundCollision = Stepping.bEnableGroundCollision;
	Settings.Stepping.MaxCollisionHeight = Stepping.MaxCollisionHeight;
	Settings.Stepping.TraceChannel = Stepping.TraceChannel;
	Settings.Stepping.OrientFootToGroundPitch = Stepping.OrientFootToGroundPitch;
	Settings.Stepping.OrientFootToGroundRoll = Stepping.OrientFootToGroundRoll;
	Settings.Stepping.IgnoredActor = ExecuteContext.GetOwningActor();
	Settings.Stepping.IgnoredComponent = ExecuteContext.GetOwningComponent();
	Settings.Stepping.World = ExecuteContext.GetWorld();
	// pelvis
	Settings.Pelvis.InputPelvisComponentSpace = Hierarchy->GetGlobalTransform(Pelvis.PelvisBone);
	Settings.Pelvis.PositionDampingHalfLife = Pelvis.PositionDampingHalfLife;
	Settings.Pelvis.RotationStiffness = Pelvis.RotationStiffness;
	Settings.Pelvis.RotationDamping = Pelvis.RotationDamping;
	Settings.Pelvis.LeadAmount = Pelvis.LeadAmount;
	Settings.Pelvis.LeadDampingHalfLife = Pelvis.LeadDampingHalfLife;
	Settings.Pelvis.BobOffset = Pelvis.BobOffset;
	Settings.Pelvis.BobStiffness = Pelvis.BobStiffness;
	Settings.Pelvis.BobDamping = Pelvis.BobDamping;
	Settings.Pelvis.OrientToGroundPitch = Pelvis.OrientToGroundPitch;
	Settings.Pelvis.OrientToGroundRoll = Pelvis.OrientToGroundRoll;
	// tick the loco motor
	Locomotor.RunSimulation(Settings);
	
	// output the resulting foot transforms
	const TArray<FLocomotorFoot*>& AllFeet = Locomotor.GetFeet();
	for (int32 FootIndex=0 ; FootIndex<AllFeet.Num(); ++FootIndex)
	{
		FeetTransforms[FootIndex] = ExecuteContext.ToVMSpace(AllFeet[FootIndex]->CurrentWorld);
	}

	// transform the pelvis bone (and propagate to all children)
	constexpr bool bInitial = false;
	constexpr bool bAffectChildren = true;
	constexpr bool bSetupUndo = false;
	constexpr bool bPrintPythonCommands = false;
	Hierarchy->SetGlobalTransform(
		Pelvis.PelvisBone,
		ExecuteContext.ToVMSpace(Locomotor.GetPelvisCurrent()),
		bInitial,
		bAffectChildren,
		bSetupUndo,
		bPrintPythonCommands);
	
	// do all debug drawing
	Debug.DrawDebug(ExecuteContext.GetDrawInterface(), Locomotor, ExecuteContext.GetToWorldSpaceTransform().Inverse());
}

void FLocomotorDebugSettings::DrawDebug(
	FRigVMDrawInterface* DrawInterface,
	const FLocomotor& Locomotor,
	const FTransform& WorldToGlobal) const
{
	{
		if (DrawInterface == nullptr || !bDrawDebug)
		{
			return;
		}

		const TArray<FLocomotorFoot*>& Feet = Locomotor.GetFeet();
		int32 FootIndex = 0;
		for (const FLocomotorFoot* Foot : Feet)
		{
			const float FootPointSize = Scale * 0.35f;
			
			if (bDrawCurrentFeet)
			{
				// draw current transform
				DrawInterface->DrawAxes(WorldToGlobal, Foot->CurrentWorld, FootPointSize, Thickness);
			}

			if (bDrawCurrentFeetFlat)
			{
				// draw current flat transform
				DrawInterface->DrawPoint(WorldToGlobal, Foot->CurrentWorldFlatPosition, FootPointSize, FLinearColor::Blue);
			}

			if (bDrawCurrentFeetTarget)
			{
				// draw current flat target
				DrawInterface->DrawPoint(WorldToGlobal, Foot->CurrentTargetWorld.GetLocation(), FootPointSize, FLinearColor::Green);
			}

			if (bDrawPrevFeetTarget)
			{
				// draw prev target transform
				DrawInterface->DrawPoint(WorldToGlobal, Foot->PlantedWorld.GetLocation(), FootPointSize, FLinearColor::Yellow);
				DrawInterface->DrawLine(
					WorldToGlobal,
					Foot->PlantedWorld.GetLocation(),
					Foot->CurrentTargetWorld.GetLocation(),
					FLinearColor::Yellow, Thickness);
				DrawInterface->DrawLine(
					WorldToGlobal,
					Foot->CurrentTargetWorld.GetLocation(),
					Foot->FinalTargetWorld.GetLocation(),
					FLinearColor::Gray, Thickness);
			}
			
			if (bDrawFinalFeetTarget)
			{
				// draw final target transform
				DrawInterface->DrawPoint(WorldToGlobal, Foot->FinalTargetWorld.GetLocation(), FootPointSize, FLinearColor::Black);
			}

			if (bDrawFeetCollision)
			{
				FTransform FootNoRotation = Foot->CurrentWorld;
				FootNoRotation.SetRotation(FQuat::Identity);
				DrawInterface->DrawCircle(
					WorldToGlobal,
					FootNoRotation,
					Foot->Settings.CollisionRadius * Locomotor.GetSettings().Stepping.FootCollisionGlobalScale,
					FLinearColor::White,
					Thickness,
					12);
			}

			++FootIndex;
		}

		// draw the phase circle with arrow that rotates around with the phase
		if (bDrawPhaseCircle)
		{
			const float Radius = Scale * 2.0f;
			
			// draw goal circle
			const FLocomotorInputSettings& Settings = Locomotor.GetSettings();
			FTransform RootGoalNoRotation = Settings.CurrentWorldRootGoal * WorldToGlobal;
			RootGoalNoRotation.SetRotation(FQuat::Identity);
			DrawInterface->DrawCircle(
				FTransform::Identity,
				RootGoalNoRotation,
				Radius,
				FLinearColor::Black,
				Thickness*2.f,
				24 /*num sides*/);

			// draw phase line
			const float AngleRadians = FMath::Lerp(0.0f, 2*PI, Locomotor.GetPhaseCurrent());
			const FVector ArrowDirection = FVector(FMath::Cos(AngleRadians), FMath::Sin(AngleRadians), 0.0f) * Radius;
			DrawInterface->DrawLine(
					FTransform::Identity,
					RootGoalNoRotation.GetLocation(),
					RootGoalNoRotation.GetLocation() + ArrowDirection,
					FLinearColor::Green, Thickness);

			// draw forward arrow
			const FVector ArrowSide = FVector(0.0f, 3.0f, 0.0f);
			DrawInterface->DrawArrow(
				Settings.CurrentWorldRootGoal * WorldToGlobal,
				FVector::RightVector * Radius,
				ArrowSide,
				FLinearColor::Black,
				Thickness*2.f);
		}

		if (bDrawBody)
		{
			FTransform BodyCurrent = Locomotor.GetBodyCurrent();
			FTransform BodyTarget = Locomotor.GetBodyTarget();
			FTransform PelvisCurrent = Locomotor.GetPelvisCurrent();
			BodyCurrent.SetScale3D(BodyCurrent.GetScale3D() * Scale * 0.4f);
			BodyTarget.SetScale3D(BodyTarget.GetScale3D() * Scale * 0.2f);
			PelvisCurrent.SetScale3D(PelvisCurrent.GetScale3D() * Scale * 0.5f);
			DrawInterface->DrawBox(FTransform::Identity,BodyCurrent, FLinearColor::Green, Thickness);
			DrawInterface->DrawBox(FTransform::Identity,BodyTarget, FLinearColor::Black, Thickness);
			DrawInterface->DrawBox(FTransform::Identity, PelvisCurrent, FLinearColor::Green, Thickness);

			DrawInterface->DrawLine(
					FTransform::Identity,
					BodyCurrent.GetLocation(),
					PelvisCurrent.GetLocation(),
					FLinearColor::Gray, Thickness);
		}
	}
}
