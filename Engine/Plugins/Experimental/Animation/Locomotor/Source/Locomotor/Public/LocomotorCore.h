// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "Engine/HitResult.h"
#include "Kismet/KismetMathLibrary.h"
#include "Math/Transform.h"
#include "LocomotorCore.generated.h"

// Daniel Holden's "Exact Damper" described here https://theorangeduck.com/page/spring-roll-call
struct FVectorDamper
{
	const FVector& Update(const FVector& TargetValue, const float DeltaTime, const float HalfLife=0.1f)
	{
		constexpr float NaturalLogTwo = 0.69314718056f;
		constexpr float Epsilon = 1e-5f;
		const float Param = 1.0f - FastNegExp((NaturalLogTwo * DeltaTime) / (HalfLife + Epsilon));
		CurrentValue.X = FMath::Lerp(CurrentValue.X, TargetValue.X, Param);
		CurrentValue.Y = FMath::Lerp(CurrentValue.Y, TargetValue.Y, Param);
		CurrentValue.Z = FMath::Lerp(CurrentValue.Z, TargetValue.Z, Param);
		return CurrentValue;
	}
	
	static float FastNegExp(float x)
	{
		// fast approximation of negative exponent
		return 1.0f / (1.0f + x + 0.48f*x*x + 0.235f*x*x*x);
	}

	void Reset(const FVector& InValue)
	{
		CurrentValue = InValue;
	}

	FVector GetCurrentValue() const
	{
		return CurrentValue;
	};
	
private:

	FVector CurrentValue;
};

struct FFloatSpring
{
	FFloatSpring() = default;

	float Update(float DeltaTime, const float& TargetValue, const float Stiffness, const float Damping)
	{
		CurrentValue = UKismetMathLibrary::FloatSpringInterp(
		CurrentValue, TargetValue, SpringState,
		Stiffness,
		Damping,
		DeltaTime, 1.0f, 0.0f);
		return CurrentValue;
	}

	void Reset(const float& InValue)
	{
		CurrentValue = InValue;
		SpringState.Reset();
	}

	float GetCurrent() const
	{
		return CurrentValue;
	}

private:

	float CurrentValue;
	FFloatSpringState SpringState;
};

struct FVectorSpring
{
	FVectorSpring() = default;

	FVector Update(float DeltaTime, const FVector& TargetValue, const float Stiffness, const float Damping)
	{
		CurrentValue = UKismetMathLibrary::VectorSpringInterp(
		CurrentValue, TargetValue, SpringState,
		Stiffness,
		Damping,
		DeltaTime, 1.0f, 0.0f);
		return CurrentValue;
	}

	void Reset(const FVector& InValue)
	{
		CurrentValue = InValue;
		SpringState.Reset();
	}

	FVector GetCurrent() const
	{
		return CurrentValue;
	}

private:

	FVector CurrentValue;
	FVectorSpringState SpringState;
};

struct FQuatSpring
{
	
	FQuatSpring() = default;

	FQuat Update(float DeltaTime, const FQuat& Target, const float Stiffness, const float Damping)
	{
		Current = UKismetMathLibrary::QuaternionSpringInterp(
			Current, Target, SpringState,
			Stiffness, Damping, DeltaTime,
			1.0f, 0.0f);
		return Current;
	}

	void Reset(const FQuat& InValue)
	{
		Current = InValue;
		SpringState.Reset();
	}

	FQuat GetCurrent() const
	{
		return Current;
	}

private:

	FQuat Current;
	FQuaternionSpringState SpringState;
};

enum class EMovementStyle : uint8
{
	Walk,
	Trot,
	Gallop,
};

struct FLocomotorMovementSettings
{
	// the smallest step to make
	float MinimumStepLength = 10.0f;
	// the maximum movement speed in cm/s
	float SpeedMax = 80.f;
	// the maximum movement speed in cm/s
	float SpeedMin = 50.f;
	// the maximum phase speed in cycles/s
	float PhaseSpeedMax = 4.f;
	// the minimum phase speed in cycles/s
	float PhaseSpeedMin = 1.0f;
	// the acceleration in cm/s/s
	float Acceleration = 100.f;
	// the acceleration in cm/s/s
	float Deceleration  = 30.f;
	// movement styles
	TArray<EMovementStyle> Styles;
	// teleportation
	bool bTeleport = false;

	//static TMap<EMovementStyle, float> StyleOffsets;
};

struct FLocomotorStepSettings
{
	// Default is 0.35. Range is 0.1 - 0.9. The percentage amount of the 0-1 stride phase where the foot is in the air.
	// At the default (0.35), the foot will plant on the ground 35% through it's total phase.
	// At lower values, the foot will spend more time on the ground as a percentage of the total stride time.
	// At higher values, each foot will spend less time on the ground as a percentage of the total stride time.
	float PercentOfStrideInAir = 0.35f;
	// Default is 0.2. Range is 0.0 - 0.9. This value is added to the "PercentOfStrideInAir" to extend the air-time while at max speed.
	// For example, if PercentOfStrideInAir is at 0.5, and the air extension is at 0.4 the foot will be in the air for 90% of the stride cycle.
	// This value is gradually blended on as the Speed accelerates between SpeedMin and SpeedMax.
	// The total "Stride In Air" is clamped internally at 95%
	float AirExtensionAtMaxSpeed = 0.2f;
	// Default is 6. Range is 0-inf. The height to raise foot (in cm) at the peak of the largest stride
	float StepHeight = 10.f;
	// Default is 0.5. Range is 0-1. How quickly the foot accelerates when lifting off the ground. At 0, it instantly moves, at 1 it eases in.
	float StepEaseIn = 0.5f;
	// Default is 0.2. Range is 0-1. How quickly the foot decelerates when planting. At 0, it instantly stops on contact, at 1 it eases into contact.
	float StepEaseOut = 0.2f;
	// Default is true. When true, feet are treated as non-overlapping circles and will not overlap each other or cross over.
	bool bEnableFootCollision = false;
	// Default is 1.0. Range is 0-inf. Global scale multiplier for all foot collision radii.
	float FootCollisionGlobalScale = 1.0f;
	// Default is true. When true, foot height is adjusted using sphere-casting against the world.
	bool bEnableGroundCollision = true;
	// Default is 30. Range is 0-inf. The maximum height in cm to place foot on-top of collision geo (may move foot up or down by this amount).
	float MaxCollisionHeight = 30.0f;
	// The 'channel' used to trace spheres into the physics world for collision detection. Determines which components to hit.
	TEnumAsByte<ETraceTypeQuery> TraceChannel;
	// Default is 0.8. Range is 0-1. When Ground Collision is enabled, this will orient the pitch of the foot to match the slope of the ground.
	float OrientFootToGroundPitch = 0.8f;
	// Default is 0.8. Range is 0-1. When Ground Collision is enabled, this will orient the roll of the foot to match the slope of the ground.
	float OrientFootToGroundRoll = 0.5f;
	// actor and component to ignore in collision queries
	const AActor* IgnoredActor = nullptr;
	const UActorComponent* IgnoredComponent = nullptr;
	// the world to collide against
	const UWorld* World = nullptr;
};

struct FLocomotorPelvisSettings
{
	// this represents the current input pose of the skeleton coming from animation
	// this allows the locomotor to adapt dynamically to changing animation
	FTransform InputPelvisComponentSpace;
	// Default is 0.05. Typical range is 0.05-1. The time (in seconds) it takes for the Pelvis to reach half way to it's position goal.
	float PositionDampingHalfLife = 0.05f;
	// Default is 2.0. Typical range is 0-20. Higher values cause pelvis to rotate more rapidly towards goal orientation.
	float RotationStiffness = 40.f;
	// Default is 2.0. Typical range is 0-20. Higher values cause pelvis rotation to dampen quickly.
	float RotationDamping = .9f;
	// Default is 3.0. Typical range is 1-3. Higher values cause pelvis to move further ahead of the feet.
	float LeadAmount = 3.f;
	// Default is 0.1. Typical range is 0.1-1. The time (in seconds) it takes for the Pelvis to reach half way to it's lead goal.
	float LeadDampingHalfLife = 0.1f;
	// Default is -3.f. Range is -inf to -inf. The amount to lower the pelvis at the minimum height in the cycle. 
	float BobOffset = -4.f;
	// Default is 150.0. Typical range is 1-200. Higher values cause pelvis to bob more rapidly towards target height.
	float BobStiffness = 150.f;
	// Default is 0.9. Typical range is 0-2. Higher values cause pelvis bob to dampen quickly.
	float BobDamping = 1.25f;
	// Default is -0.3. Typical range is -1 to 1. Pitch the pelvis up/down with ground. 0 is no effect, 1/-1 matches ground orientation. Use negative values for bipeds, positive for quadrupeds.
	float OrientToGroundPitch = -0.3f;
	// Default is -0.3. Typical range is -1 to 1. Roll the pelvis side-to-side with ground. 0 is no effect, 1/-1 matches ground orientation. Use negative values for bipeds, positive for quadrupeds.
	float OrientToGroundRoll = -0.3f;
};

struct FLocomotorSpineSettings
{
	// Default is 30. The maximum angle (in degrees) to lean the spine in the direction of travel.
	float LeanAngleMax = 30.0f;
	// Default is 150.0. Typical range is 1-200. Higher values cause spine to lean more rapidly towards target direction.
	float LeanStiffness = 150.f;
	// Default is 0.9. Typical range is 0-2. Higher values cause spine leaning to dampen quickly.
	float LeanDamping = 0.9f;
};

struct FLocomotorHeadSettings
{
	// Default is 30. The maximum angle (in degrees) to rotate the head to look at the goal.
	float AngleMax = 30.0f;
	// Default is 150.0. Typical range is 1-200. Higher values cause spine to lean more rapidly towards target direction.
	float Stiffness = 150.f;
	// Default is 0.9. Typical range is 0-2. Higher values cause spine leaning to dampen quickly.
	float Damping = 0.9f;
};

struct FLocomotorFootSettings
{
	// the radius in cm to avoid feet penetrations
	float CollisionRadius = 10.f;
	// Default is (0, 0, 50). The max rotation to peel the heel before lift off. This is in the local space of the ankle bone.
	FVector MaxHeelPeel = FVector(0.f, 0.f, 50.f);
	// either 0.0f or 0.5f depending on order in set (alternates Left/Right/Left/Right with values 0 / 0.5 / 0 / 0.5 etc...)
	float StaticPhaseOffset = 0.f;
	// applies a static offset that shifts feet in the local space of the foot bone in the reference pose
	FVector StaticLocalOffset = FVector::ZeroVector;
	// the index of the set this foot belongs to
	int32 FootSetIndex = 0;
};

struct FLocomotorInputSettings
{
	// the current target goal location in world space
	FTransform CurrentWorldRootGoal;
	// time in seconds since last update
	float DeltaTime = 0.16f;
	// all settings related to overall movement and strides
	FLocomotorMovementSettings Movement;
	// all settings related to steps
	FLocomotorStepSettings Stepping;
	// pelvis settings
	FLocomotorPelvisSettings Pelvis;
	// spine settings
	FLocomotorSpineSettings Spine;
};

struct FPlaneConstraint
{
	FPlaneConstraint() : Active(false), Normal(FVector::UpVector) {}
	
	bool Active;
	FVector Normal;
};

struct FLocomotorFoot
{
	FLocomotorFoot(
		const FTransform& InitialWorldRootGoal,
		const FTransform& InitialWorldFoot,
		const FLocomotorFootSettings& InSettings);

	FLocomotorFootSettings Settings;

	// the current phase gradually approaches this value, we don't directly set the current phase to avoid foot popping  
	float TargetPhase = 0.f;
	// the current phase of this foot to use for animation
	float CurrentPhase = 0.f;
	// the point in the foot's phase when the swing begins (variable with speed/frequency)
	float FootPhaseWhenSwingEnds = 0.5f;
	// whether this foot is close enough to the final goal to stop animating
	bool bAtRest = true;
	// current state of the foot (either swinging or planted)
	bool bInSwingPhase = false;
	// only true on the tick when this foot transitions to bInSwingPhase=true
	bool bUnplantingThisTick = false;
	// does this foot currently have a target beyond the min step threshold?
	bool bWantsToStep = false;
	// the distance from the planted location to the final target
	FVector PrevToFinalTargetNorm;
	float PrevToFinalTargetDistance;

	// the initial transform relative to the initial root goal
	FTransform InitialRelativeToRootGoal;
	// the initial transform relative to the initial body
	FTransform InitialRelativeToBody;
	// initial transform
	FTransform InitialWorld;

	// the max height of the foot for the current stride (calculated once when unplanting)
	float CurrentStrideHeight;
	// the neutral location at the start of a step, may be different from the actual planted location
	FTransform StrideOriginWorld;
	// the prev foot plant in world space
	FTransform PlantedWorld;
	// the current target location to step towards
	FTransform CurrentTargetWorld;
	// a heavily damped spring for continuously updating target
	FVectorSpring CurrentTargetSpring;
	// the final destination location for foot
	FTransform FinalTargetWorld;
	
	// the current world position of this foot with no height added
	FVector CurrentWorldFlatPosition;
	// the current world position of this foot with no height added and no ease in/out applied
	FVector CurrentWorldFlatPositionNoEase;
	// the current world transform of this foot
	FTransform CurrentWorld;
	// the world transform of this foot at the end of the previous tick
	FTransform PrevWorld;

	// store current height offset for bobbing
	float CurrentHeightOffset;

	// rotation is smoothly blended through a spring
	FQuatSpring CurrentRotationSpring;
};

struct FLocomotorFootSet
{
	FLocomotorFootSet(const float InPhaseOffset, int32 InSetIndex) : PhaseOffset(InPhaseOffset), SetIndex(InSetIndex){};

	// add a foot, returns the index of the foot or -1 if no set found
	int32 AddFoot(
		const FTransform& InitialWorldRootGoal,
		const FTransform& InitialWorldFoot,
		const FLocomotorFootSettings& InSettings);

	// get access to the feet in this set
	TArray<FLocomotorFoot>& GetFeet() {return Feet; };
	const TArray<FLocomotorFoot>& GetFeet() const {return Feet; };

	// get number of feet in this set
	int32 GetNumFeet() const { return Feet.Num(); };

	// get the current offset of this set
	float GetPhaseOffset() const { return PhaseOffset; };
	
private:
	
	// the 0-1 phase offset for this set of feet (relative to the global phase)
	float PhaseOffset = 0.f;
	// the array of feet belonging to this set
	TArray<FLocomotorFoot> Feet;
	// the index of this set
	int32 SetIndex = 0;
};

struct FLocomotorHead
{
	FLocomotorHead() = default;

	void Initialize(const FTransform& InInitialWorld);
	
	// the initial world transform of the head bone
	FTransform InitialWorld;
	FTransform CurrentWorld;
	FQuatSpring RotationSpring;

	bool bInitialized = false;
};

struct FLocomotorPelvis
{
	FLocomotorPelvis() = default;

	void Initialize(const TArray<FLocomotorFoot*>& AllFeet, const FTransform& InitialRootGoalWorld);

	void CalculateBodyTransform(
		const TArray<FLocomotorFoot*> AllFeet, 
		const FTransform& RootGoal,
		FTransform& OutBodyTransform);
	
	// the initial world transform of the pelvis
	FTransform InitialWorld;
	FTransform InitialRelativeToRootGoal;
	FTransform CurrentWorld;
	
	// the transform of the "body"; this is a separately maintained transform representing the center of mass
	// which is different than the pelvis location which is usually far from the center of the body in quadrupeds
	FTransform InitialBodyWorld;
	FTransform CurrentBodyWorld;
	FTransform PreviousBodyTargetWorld;
	FTransform CurrentBodyTargetWorld;
	FVector CurrentBodyLead;
	FVectorDamper BodyLeadDamper;
	FQuatSpring BodyRotationSpring;
	FVectorDamper PelvisPositionDamper;

	// height offset (bob) is tracked separately and applied additively through a spring
	float CurrentHeightOffset;
	FFloatSpring BobOffsetSpring;
};

USTRUCT()
struct FLocomotor
{
	GENERATED_BODY()
	
public:
	
	FLocomotor() = default;

	// initialize all internal data structures (can be called multiple times)
	void Reset(
		const FTransform& InInitialRootGoalWorld,
		const FTransform& InInitialWorldPelvis);

	// add a set of feet, returns the foot set
	int32 AddFootSet(const float PhaseOffset);

	// add a foot, returns the index of the foot or -1 if no set found
	int32 AddFootToSet(
		const int32 FootSetIndex,
		const FTransform& InitialWorldFoot,
		const FLocomotorFootSettings& InSettings);

	// set the head transform and settings
	void SetHead(const FTransform& InitialWorldTransform);

	// update the simulation state and generate new foot transforms
	void RunSimulation(const FLocomotorInputSettings& Settings);

	// must have at least one set of feet to progress
	bool HasFeet() const { return !AllFeet.IsEmpty(); };

	// read-only access to feet
	const TArray<FLocomotorFootSet>& GetFootSets() const { return FootSets; };

	// read-only access to pelvis transform
	const FTransform& GetPelvisCurrent() const { return Pelvis.CurrentWorld; };

	// read-only access to body transform
	const FTransform& GetBodyCurrent() const { return Pelvis.CurrentBodyWorld; };

	// read-only access to pelvis transform
	const FTransform& GetBodyTarget() const { return Pelvis.CurrentBodyTargetWorld; };

	// get the current phase
	float GetPhaseCurrent() const;

	// read-only access to settings
	const FLocomotorInputSettings& GetSettings() const { return Settings; };

	// get final foot transforms
	void GetFeetCurrent(TArray<FTransform> OutTransforms) const;

	// get access to all feet
	TArray<FLocomotorFoot*>& GetFeet() {return AllFeet; };
	const TArray<FLocomotorFoot*>& GetFeet() const {return AllFeet; };

private:

	void Simulate();
	bool Initialize();
	void UpdateWorldSpeedAndPhase();
	void UpdateFeetTargets();
	void AnimateFeet();
	void UpdateBody();
	void AnimatePelvis();
	void AnimateHead();
	
	void ResolveFootToFootCollision(FLocomotorFoot& Foot);
	void PushCirclesApartInFloorPlane(
		FVector& CenterA,
		FVector& CenterB,
		float RadiusA,
		float RadiusB,
		float InvMassA,
		float InvMassB);
	FVector ProjectToGroundWithSphereCast(const FLocomotorFoot& Foot, FTransform& TransformToProject);

	float EaseInOutNorm(const float Input, float EaseInAmount, float EaseOutAmount) const;
	static float WrapPhaseInRange(const float PhaseToWrap);
	static float BlendTowardsTargetPhase(const float CurrentPhase, const float TargetPhase, const float PhaseBlendRate);
	float GetPhaseOffsetForSetFromMovementStyle(int32 SetIndex);

	bool CastSphere(const FVector& Start, const FVector& End, float SphereRadius, FHitResult& OutHitResult);

	void CalcPitchRollFromNormal(
		const FVector& Fwd,
		const FVector& Up,
		const FVector& Normal,
		FVector& OutSide,
		FQuat& OutPitchRotation,
		FQuat& OutRollRotation);
	
	// the current settings (updated each frame)
	FLocomotorInputSettings Settings;
	// the root goal in world space at the start of the simulation
	FTransform InitialRootGoalWorld;
	// current state of each foot being animated
	TArray<FLocomotorFootSet> FootSets;
	// pointers to all feet in all sets (for convenience)
	TArray<FLocomotorFoot*> AllFeet;
	// pelvis data
	FLocomotorPelvis Pelvis;
	// head data
	FLocomotorHead Head;
	
	// global phase and movement speed (dynamically adjusted during simulation)
	float CurrentGlobalPhase;
	float CurrentPhaseSpeed;
	float CurrentSpeed;
	float CurrentPercentOfMaxSpeed;
	float CurrentStrideLength;
	bool bFullyAtRest = true;
	bool bAccelerating = true;
	float AccumulatedTimeToSimulate;

	// whether the locomotor has cached it's internal state on first update
	bool bPostInitialized = false;
};
