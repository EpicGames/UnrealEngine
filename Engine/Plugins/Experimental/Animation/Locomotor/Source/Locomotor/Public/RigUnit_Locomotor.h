// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LocomotorCore.h"
#include "Units/RigUnitContext.h"
#include "Units/Highlevel/RigUnit_HighlevelBase.h"

#include "RigUnit_Locomotor.generated.h"

USTRUCT()
struct LOCOMOTOR_API FLocomotorDebugSettings
{
	GENERATED_BODY()

	FLocomotorDebugSettings() :
		bDrawDebug(true),
		bDrawBody(true),
		bDrawPhaseCircle(true),
		bDrawCurrentFeet(true),
		bDrawCurrentFeetFlat(true),
		bDrawCurrentFeetTarget(true),
		bDrawPrevFeetTarget(true),
		bDrawFinalFeetTarget(true),
		bDrawFeetCollision(true),
		Scale(25.0f),
		Thickness(0.5f) {}

	UPROPERTY(meta = (Input))
	bool bDrawDebug;

	UPROPERTY(meta = (Input))
	bool bDrawBody;

	UPROPERTY(meta = (Input))
	bool bDrawPhaseCircle;
	
	UPROPERTY(meta = (Input))
	bool bDrawCurrentFeet;

	UPROPERTY(meta = (Input))
	bool bDrawCurrentFeetFlat;

	UPROPERTY(meta = (Input))
	bool bDrawCurrentFeetTarget;
	
	UPROPERTY(meta = (Input))
	bool bDrawPrevFeetTarget;

	UPROPERTY(meta = (Input))
	bool bDrawFinalFeetTarget;

	UPROPERTY(meta = (Input))
	bool bDrawFeetCollision;
	
	UPROPERTY(meta = (Input, ClampMin = "0.0", UIMin = "0.01", UIMax = "100.0"))
	float Scale;

	UPROPERTY(meta = (Input, ClampMin = "0.0", UIMin = "0.01", UIMax = "100.0"))
	float Thickness;
	
	void DrawDebug(FRigVMDrawInterface* DrawInterface, const FLocomotor& Locomotor, const FTransform& WorldToGlobal) const;
};

UENUM()
enum class ELocomotorMovementStyle : uint8
{
	Walk,
	Trot,
	Gallop,
};

USTRUCT()
struct LOCOMOTOR_API FMovementSettings
{
	GENERATED_BODY()

	FMovementSettings() : 
		MinimumStepLength(10.f),
		SpeedMax(80.f),
		SpeedMin(50.f),
		Acceleration(100.f),
		Deceleration(30.f),
		GlobalTimeScale(1.f),
		bTeleport(false){}
	
	// the length of the smallest step to take in cm
	UPROPERTY(meta = (Input))
	float MinimumStepLength = 10.f;
	
	// the maximum movement speed in cm/s
	UPROPERTY(meta = (Input))
	float SpeedMax = 80.f;

	// the minimum movement speed in cm/s
	UPROPERTY(meta = (Input))
	float SpeedMin = 50.f;

	// the maximum phase speed in cycles/s
	UPROPERTY(meta = (Input))
	float PhaseSpeedMax = 4.f;
	
	// the minimum phase speed in cycles/s
	UPROPERTY(meta = (Input))
	float PhaseSpeedMin = 1.0f;

	// the acceleration in cm/s/s (ie, how quickly the movement speed increases to reach the goal)
	UPROPERTY(meta = (Input))
	float Acceleration = 100.f;

	// the acceleration in cm/s/s (ie, how quickly the movement speed decreases when approaching the goal)
	UPROPERTY(meta = (Input))
	float Deceleration = 30.f;

	// adjust global speed of the simulation
	UPROPERTY(meta = (Input))
	float GlobalTimeScale = 1.f;

	// reset the simulation. this can be used to teleport the locomotor.
	UPROPERTY(meta = (Input))
	bool bTeleport;

	// The the movement styles to cycle through based on speed
	UPROPERTY(meta = (Input))
	TArray<ELocomotorMovementStyle> Styles;
};

USTRUCT()
struct LOCOMOTOR_API FStepSettings
{
	GENERATED_BODY()

	FStepSettings() :
		PercentOfStrideInAir(0.35f),
		AirExtensionAtMaxSpeed(0.2f),
		StepHeight(6.f),
		StepEaseIn(0.5f),
		StepEaseOut(0.2f),
		bEnableFootCollision(true),
		FootCollisionGlobalScale(1.0f),
		bEnableGroundCollision(true),
		MaxCollisionHeight(30.0f),
		TraceChannel(ETraceTypeQuery::TraceTypeQuery1),
		OrientFootToGroundPitch(0.8f),
		OrientFootToGroundRoll(0.5f){}
	
	// Default is 0.35. Range is 0.1 - 0.9. The percentage amount of the 0-1 stride phase where the foot is in the air.
	// At the default (0.35), the foot will plant on the ground 35% through it's total phase.
	// At lower values, the foot will spend more time on the ground as a percentage of the total stride time.
	// At higher values, each foot will spend less time on the ground as a percentage of the total stride time.
	UPROPERTY(meta = (Input, UIMin = "0.1", UIMax=".9", ClampMin="0.1", ClampMax=".9"))
	float PercentOfStrideInAir = 0.35f;
	// Default is 0.2. Range is 0.0 - 0.9. This value is added to the "PercentOfStrideInAir" to extend the air-time while at max speed.
	// For example, if PercentOfStrideInAir is at 0.5, and the air extension is at 0.4 the foot will be in the air for 90% of the stride cycle.
	// This value is gradually blended on as the Speed accelerates between SpeedMin and SpeedMax.
	// The total "Stride In Air" is clamped internally at 95%
	UPROPERTY(meta = (Input, UIMin = "0.1", UIMax=".9", ClampMin="0.1", ClampMax=".9"))
	float AirExtensionAtMaxSpeed = 0.2f;
	
	// Default is 6. Range is 0-inf. The height to raise foot (in cm) at the peak of the largest stride
	UPROPERTY(meta = (Input, UIMin = "0.1", UIMax="50"))
	float StepHeight = 6.f;
	// Default is 0.5. Range is 0-1. How quickly the foot accelerates when lifting off the ground. At 0, it instantly moves, at 1 it eases in.
	UPROPERTY(meta = (Input, UIMin = "0", UIMax="1", ClampMin="0", ClampMax="1"))
	float StepEaseIn = 0.5f;
	// Default is 0.2. Range is 0-1. How quickly the foot decelerates when planting. At 0, it instantly stops on contact, at 1 it eases into contact.
	UPROPERTY(meta = (Input, UIMin = "0", UIMax="1", ClampMin="0", ClampMax="1"))
	float StepEaseOut = 0.2f;
	
	// Default is true. When true, feet are treated as non-overlapping circles and will not overlap each other or cross over.
	UPROPERTY(meta = (Input))
	bool bEnableFootCollision = true;
	// Default is 1.0. Range is 0-inf. Global scale multiplier for all foot collision radii.
	UPROPERTY(meta = (Input))
	float FootCollisionGlobalScale = 1.0f;
	// Default is true. When true, foot height is adjusted using sphere-casting against the world.
	UPROPERTY(meta = (Input))
	bool bEnableGroundCollision = true;
	// Default is 30. Range is 0-inf. The maximum height in cm to place foot on-top of collision geo (may move foot up or down by this amount).
	UPROPERTY(meta = (Input, UIMin = "0"))
	float MaxCollisionHeight = 30.0f;
	// The 'channel' used to trace spheres into the physics world for collision detection. Determines which components to hit.
	UPROPERTY(meta = (Input))
	TEnumAsByte<ETraceTypeQuery> TraceChannel = ETraceTypeQuery::TraceTypeQuery1;
	// Default is 0.8. Range is 0-1. When Ground Collision is enabled, this will orient the pitch of the foot to match the slope of the ground.
	UPROPERTY(meta = (Input, UIMin = "0", UIMax="1", ClampMin="0", ClampMax="1"))
	float OrientFootToGroundPitch = 0.8f;
	// Default is 0.8. Range is 0-1. When Ground Collision is enabled, this will orient the roll of the foot to match the slope of the ground.
	UPROPERTY(meta = (Input, UIMin = "0", UIMax="1", ClampMin="0", ClampMax="1"))
	float OrientFootToGroundRoll = 0.5f;
};

USTRUCT()
struct LOCOMOTOR_API FPelvisSettings
{
	GENERATED_BODY()

	FPelvisSettings() :
		PelvisBone(FRigElementKey(NAME_None, ERigElementType::Bone)),
		PositionDampingHalfLife(0.1f),
		RotationStiffness(40.0f),
		RotationDamping(0.9f),
		LeadAmount(2.0f),
		LeadDampingHalfLife(0.1f),
		BobOffset(-8.f),
		BobStiffness(40.f),
		BobDamping(.9f),
		OrientToGroundPitch(-0.3f),
		OrientToGroundRoll(-0.3f){};
	
	// Usually the Pelvis or Hip bone. The bone that translates the whole body (not necessarily the root of the entire hierarchy).
	UPROPERTY(meta = (Input))
	FRigElementKey PelvisBone = FRigElementKey(NAME_None, ERigElementType::Bone);
	
	// Default is 0.1. Typical range is 0.1-1. The time (in seconds) it takes for the Pelvis to reach half way to it's goal.
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "10.0"))
	float PositionDampingHalfLife = 0.1f;

	// Default is 40.0. Typical range is 1-100. Higher values cause pelvis to rotate more rapidly towards goal orientation.
	UPROPERTY(meta = (Input, UIMin = "0.1", UIMax = "100.0"))
	float RotationStiffness = 40.f;
	// Default is 2.0. Typical range is 0-2. Higher values cause pelvis rotation to dampen quickly.
	UPROPERTY(meta = (Input, UIMin = "0.1", UIMax = "2.0"))
	float RotationDamping = .9f;

	// Default is 2.0. Typical range is 1-3. Higher values cause pelvis to move further ahead of the feet.
	UPROPERTY(meta = (Input, UIMin = "0.1", UIMax = "20.0"))
	float LeadAmount = 2.f;
	// Default is 0.1. Typical range is 0.1-1. The time (in seconds) it takes for the Pelvis to reach half way to it's lead goal.
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "10.0"))
	float LeadDampingHalfLife = 0.1f;
	
	// Default is 10.f. Range is 0-inf. The amount to lower the pelvis at the minimum height in the cycle.
	UPROPERTY(meta = (Input, UIMin = "0"))
	float BobOffset = -8.f;
	// Default is 40.0. Typical range is 1-100. Higher values cause pelvis to bob more rapidly towards target height.
	UPROPERTY(meta = (Input, UIMin = "0.1", UIMax = "20.0"))
	float BobStiffness = 40.f;
	// Default is 0.9. Typical range is 0-2. Higher values cause pelvis bob to dampen quickly.
	UPROPERTY(meta = (Input, UIMin = "0.1", UIMax = "20.0"))
	float BobDamping = .9f;

	// Default is -0.3. Typical range is -1 to 1. Pitch the pelvis up/down with ground. 0 is no effect, 1/-1 matches ground orientation. Use negative values for bipeds, positive for quadrupeds.
	UPROPERTY(meta = (Input, UIMin = "-1", UIMax = "1.0"))
	float OrientToGroundPitch = -0.3f;
	// Default is -0.3. Typical range is -1 to 1. Roll the pelvis side-to-side with ground. 0 is no effect, 1/-1 matches ground orientation. Use negative values for bipeds, positive for quadrupeds.
	UPROPERTY(meta = (Input, UIMin = "-1", UIMax = "1.0"))
	float OrientToGroundRoll = -0.3f;
};

USTRUCT()
struct LOCOMOTOR_API FSpineSettings
{
	GENERATED_BODY()

	FSpineSettings() :
		SpineBone(FRigElementKey(NAME_None, ERigElementType::Bone)),
		PitchAngleMax(30.0f),
		PitchStiffness(150.f),
		PitchDamping(0.9f){}
	
	// The base spine bone. Usually directly below the Pelvis bone. The bone that rotates to lean the whole spine.
	UPROPERTY(meta = (Input))
	FRigElementKey SpineBone = FRigElementKey(NAME_None, ERigElementType::Bone);

	// Default is 30. The maximum angle (in degrees) to lean the spine in the direction of travel.
	UPROPERTY(meta = (Input, UIMin = "0.1", UIMax = "180.0"))
	float PitchAngleMax = 30.0f;
	
	// Default is 150.0. Typical range is 1-200. Higher values cause spine to lean more rapidly towards target direction.
	UPROPERTY(meta = (Input, UIMin = "0.1", UIMax = "200.0"))
	float PitchStiffness = 150.f;
	
	// Default is 0.9. Typical range is 0-2. Higher values cause spine leaning to dampen quickly.
	UPROPERTY(meta = (Input, UIMin = "0.1", UIMax = "20.0"))
	float PitchDamping = 0.9f;
};

USTRUCT()
struct LOCOMOTOR_API FHeadSettings
{
	GENERATED_BODY()

	FHeadSettings() :
		HeadBone(FRigElementKey(NAME_None, ERigElementType::Bone)),
		AngleMax(30.0f),
		Stiffness(150.f),
		Damping(0.9f){}
	
	// The base spine bone. Usually directly below the Pelvis bone. The bone that rotates to lean the whole spine.
	UPROPERTY(meta = (Input))
	FRigElementKey HeadBone = FRigElementKey(NAME_None, ERigElementType::Bone);

	// Default is 30. The maximum angle (in degrees) to rotate the head to look at the goal.
	UPROPERTY(meta = (Input, UIMin = "0.1", UIMax = "180.0"))
	float AngleMax = 30.0f;
	
	// Default is 150.0. Typical range is 1-200. Higher values cause spine to lean more rapidly towards target direction.
	UPROPERTY(meta = (Input, UIMin = "0.1", UIMax = "200.0"))
	float Stiffness = 150.f;
	
	// Default is 0.9. Typical range is 0-2. Higher values cause spine leaning to dampen quickly.
	UPROPERTY(meta = (Input, UIMin = "0.1", UIMax = "20.0"))
	float Damping = 0.9f;
};


USTRUCT()
struct LOCOMOTOR_API FFootSettings
{
	GENERATED_BODY()

	FFootSettings() = default;

	// The bone at the END of the limb (typically the BALL of the foot). This is where the output goal originates.
	UPROPERTY(meta = (Input))
	FRigElementKey AnkleBone;

	UPROPERTY(meta = (Input))
	float CollisionRadius = 10.f;

	// Default is (0, 0, 50). The max rotation to peel the heel before lift off. This is in the local space of the ankle bone.
	UPROPERTY(meta = (Input))
	FVector MaxHeelPeel = FVector(0.f, 0.f, 50.f);

	// applies a static offset that shifts feet in the local space of the foot bone in the reference pose
	UPROPERTY(meta = (Input))
	FVector StaticLocalOffset = FVector::ZeroVector;
};

USTRUCT()
struct LOCOMOTOR_API FFootSet
{
	GENERATED_BODY()

	FFootSet() = default;

	// All the feet belonging to this set. Feet in this set alternate phases (ie Left/Right/Left/Right).
	UPROPERTY(meta = (Input))
	TArray<FFootSettings> Feet;

	// the 0-1 phase offset for this set of feet (relative to the global phase)
	UPROPERTY(meta = (Input, UIMin = "0.0", UIMax = "1.0"))
	float PhaseOffset = 0.f;
};

/*
 * A node providing access to the locomotor functionality for foot step generation.
 */
USTRUCT(meta=(DisplayName="Locomotor", Category="Simulation", Keywords="Walk Step Walker Walking Locomotion Procedural"))
struct LOCOMOTOR_API FRigUnit_Locomotor: public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	virtual void Execute() override;

	FRigUnit_Locomotor() :
		RootControl(NAME_None),
		CachedRootElement(FCachedRigElement()){}

	// list of limbs to walk on.
	UPROPERTY(meta = (Output, DefaultArraySize = 2))
	TArray<FTransform> FeetTransforms;

	// the target location and rotation, in world space
	// the locomotor will generate steps to pull the character towards this location
	// footsteps will be oriented match the root goal's final orientation
	UPROPERTY(meta = (Input, CustomWidget = "ControlName" ))
	FName RootControl;
	// Used to cache the root element
	UPROPERTY()
	FCachedRigElement CachedRootElement;

	// the movement settings
	UPROPERTY(meta = (Input))
	FMovementSettings Movement;

	// the set of feet belonging to the creature
	UPROPERTY(meta = (Input, Constant))
	TArray<FFootSet> FootSets;

	// the global step settings
	UPROPERTY(meta = (Input))
	FStepSettings Stepping;

	// the Pelvis or Hip bone to move with the feet
	UPROPERTY(meta = (Input))
	FPelvisSettings Pelvis;

	// TODO
	// the spine settings
	//UPROPERTY(meta = (Input))
	//FSpineSettings Spine;

	// TODO
	// the head settings
	//UPROPERTY(meta = (Input))
	//FHeadSettings Head;

	// adjust debug drawing.
	UPROPERTY(meta = (Input))
	FLocomotorDebugSettings Debug;

	// the locomotion simulator
	UPROPERTY(Transient)
	FLocomotor Locomotor;
};
